{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Derived Data Build Controller",
	"Description" : "Adds support for shader compiling distribution using the Derived Data Build API",
	"Category" : "Build Distribution",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"IsExperimentalVersion": true,
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,

	"Modules" :
	[
		{
			"Name" : "DerivedDataBuildController",
			"Type" : "Editor",
			"LoadingPhase" : "EarliestPossible",
			"PlatformAllowList": ["Win64", "Mac", "Linux"]
		}
	],

	"SupportedTargetPlatforms": ["Win64", "Mac", "Linux"]
}
