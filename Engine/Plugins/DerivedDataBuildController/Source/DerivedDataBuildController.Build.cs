// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class DerivedDataBuildController : ModuleRules
{
	public DerivedDataBuildController(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"DerivedDataCache",
			"DistributedBuildInterface",
		});
	}
}
