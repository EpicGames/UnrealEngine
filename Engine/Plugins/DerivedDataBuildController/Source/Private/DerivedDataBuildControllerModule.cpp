// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compression/CompressedBuffer.h"
#include "Compression/OodleDataCompression.h"
#include "DerivedDataBuild.h"
#include "DerivedDataBuildAction.h"
#include "DerivedDataBuildInputs.h"
#include "DerivedDataBuildOutput.h"
#include "DerivedDataBuildSession.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataValue.h"
#include "DistributedBuildControllerInterface.h"
#include "Features/IModularFeatures.h"
#include "IO/IoHash.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

// Used to register the remote function.
#include "DerivedDataBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"

DEFINE_LOG_CATEGORY_STATIC(LogDerivedDataBuildController, Log, All);

class FDerivedDataBuildControllerModule final : public IDistributedBuildController , public TSharedFromThis<FDerivedDataBuildControllerModule>
{
public:
	FDerivedDataBuildControllerModule() = default;
	~FDerivedDataBuildControllerModule() final = default;

	// IModuleInterface

	void StartupModule() final;
	void ShutdownModule() final;

	// IDistributedBuildController

	bool SupportsLocalWorkers() final { return true; }
	bool SupportsVirtualFiles() const final { return true; }

	inline int32 GetMaxNumLocalWorkers() const { return MaxNumLocalWorkers; }
	void SetMaxLocalWorkers(int32 InMaxNumLocalWorkers) final { MaxNumLocalWorkers = InMaxNumLocalWorkers; }

	void InitializeController() final;
	bool IsSupported() final;

	const FString GetName() final { return "Derived Data Build Controller"; }

	FString CreateUniqueFilePath() final;
	bool PollStats(FDistributedBuildStats& OutStats) final;
	TFuture<FDistributedBuildTaskResult> EnqueueTask(const FTaskCommandData& CommandData) final;

	static const UE::FUtf8SharedString& GetFunctionName()
	{
		static const UE::FUtf8SharedString Name(ANSITEXTVIEW("CompileShaderJobs"));
		return Name;
	}

private:
	bool bSupported = false;
	bool bControllerInitialized = false;
	
	TAtomic<int32> MaxNumLocalWorkers = -1;

	UE::DerivedData::FOptionalBuildSession Session;
};

IMPLEMENT_MODULE(FDerivedDataBuildControllerModule, DerivedDataBuildController);

namespace DerivedDataBuildControllerModule
{
	static constexpr int32 SubFolderCount = 32;
}

static bool IsDerivedDataBuildControllerEnabled()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("NoDDBShaderCompile")) ||
		FParse::Param(FCommandLine::Get(), TEXT("NoShaderWorker")))
	{
		return false;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("DDBShaderCompile")))
	{
		return true;
	}

	return false;
}

bool FDerivedDataBuildControllerModule::IsSupported()
{
	if (!bControllerInitialized)
	{
		bSupported = IsDerivedDataBuildControllerEnabled() && FPlatformProcess::SupportsMultithreading();
	}
	return bSupported;
}

void FDerivedDataBuildControllerModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureType(), this);
}

void FDerivedDataBuildControllerModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureType(), this);
	
	if (bControllerInitialized)
	{
		Session.Reset();
		bControllerInitialized = false;
	}
}

void FDerivedDataBuildControllerModule::InitializeController()
{
	using namespace UE;
	using namespace UE::DerivedData;

	if (ensureAlwaysMsgf(!bControllerInitialized, TEXT("Multiple initialization of Derived Data Build controller!")))
	{
		if (IsSupported())
		{
			FModuleManager::Get().LoadModuleChecked(TEXT("DerivedDataCache"));
			Session = UE::DerivedData::GetBuild().CreateSession(TEXTVIEW("DerivedDataBuildControllerModule"));
		}

		bControllerInitialized = true;
	}
}

FString FDerivedDataBuildControllerModule::CreateUniqueFilePath()
{
	// These paths are never used because of virtual files, but paths are created anyway.
	static const FString WorkingDirectory = FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("DerivedDataBuild"), FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower));
	static int32 NextFileID = 0;
	return FPaths::Combine(WorkingDirectory, FString::Printf(TEXT("%d.tmp"), NextFileID++));
}

TFuture<FDistributedBuildTaskResult> FDerivedDataBuildControllerModule::EnqueueTask(const FTaskCommandData& CommandData)
{
	using namespace UE;
	using namespace UE::DerivedData;

	check(bSupported);

	const FSharedString Name(CommandData.Description);

	IBuild& Build = GetBuild();
	FBuildActionBuilder ActionBuilder = Build.CreateAction(Name, GetFunctionName());
	FBuildInputsBuilder InputsBuilder = Build.CreateInputs(Name);

	{
		const FCompressedBuffer Input = FCompressedBuffer::Compress(FSharedBuffer::MakeView(CommandData.InputFileData));
		ActionBuilder.AddInput(ANSITEXTVIEW("Input"), Input.GetRawHash(), Input.GetRawSize());
		InputsBuilder.AddInput(ANSITEXTVIEW("Input"), Input);
	}

	for (int32 ViewIndex = 0, ViewCount = CommandData.SourceData.Num(); ViewIndex < ViewCount; ++ViewIndex)
	{
		const FCompressedBuffer Input = FCompressedBuffer::Compress(FSharedBuffer::MakeView(CommandData.SourceData[ViewIndex]),
			ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);
		TAnsiStringBuilder<32> InputName(InPlace, ANSITEXTVIEW("Virtual"), ViewIndex);
		ActionBuilder.AddInput(InputName, Input.GetRawHash(), Input.GetRawSize());
		InputsBuilder.AddInput(InputName, Input);
	}

	TPromise<FDistributedBuildTaskResult> Promise;
	TFuture<FDistributedBuildTaskResult> Future = Promise.GetFuture();

	FRequestOwner Owner(EPriority::Normal);
	Owner.KeepAlive();
	Session.Get().Build(ActionBuilder.Build(), InputsBuilder.Build(), EBuildPolicy::Build, Owner,
		[Promise = MoveTemp(Promise)](FBuildCompleteParams&& Params) mutable
		{
			FDistributedBuildTaskResult Result;
			Result.bCompleted = Params.Status != EStatus::Canceled;

			static const FValueId OutputId = FValueId::FromName(ANSITEXTVIEW("Output"));
			if (const FValueWithId& Value = Params.Output.GetValue(OutputId))
			{
				TArray<uint8> OutputData;
				if (const FCompressedBuffer& CompressedData = Value.GetData())
				{
					const uint64 RawSize = CompressedData.GetRawSize();
					if (RawSize <= MAX_int32)
					{
						OutputData.SetNumUninitialized((int32)RawSize);
						if (CompressedData.TryDecompressTo(OutputData))
						{
							Result.OutputData = MoveTemp(OutputData);
						}
					}
				}
			}

			Result.ReturnCode = Result.OutputData.IsEmpty() ? 1 : 0;
			Promise.SetValue(MoveTemp(Result));
		});

	return MoveTemp(Future);
}

bool FDerivedDataBuildControllerModule::PollStats(FDistributedBuildStats& OutStats)
{
	return false;
}

class FCompileShaderJobsBuildFunction final : public UE::DerivedData::IBuildFunction
{
public:
	const UE::FUtf8SharedString& GetName() const final
	{
		return FDerivedDataBuildControllerModule::GetFunctionName();
	}

	FGuid GetVersion() const final
	{
		// This version must match ShaderBuildWorker.cpp.
		// Version is not yet meaningfully derived from the shader build code,
		// which means that it cannot yet reliably distinguish two build workers.
		static const FGuid Version(TEXT("83027356-2cf7-41ca-aba5-c81ab0ff2129"));
		return Version;
	}

	void Configure(UE::DerivedData::FBuildConfigContext& Context) const final
	{
		// Register the function for remote builds only.
		// Building in-process will eventually be supported by a build function
		// that builds one shader at a time instead of batches.
		Context.SetBuildPolicyMask(~UE::DerivedData::EBuildPolicy::BuildLocal);
	}

	void Build(UE::DerivedData::FBuildContext& Context) const final
	{
		checkNoEntry();
	}
};

static UE::DerivedData::TBuildFunctionFactory<FCompileShaderJobsBuildFunction> CompileShaderJobsFunction;
